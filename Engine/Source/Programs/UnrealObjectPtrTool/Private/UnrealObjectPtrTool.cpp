// Copyright Epic Games, Inc. All Rights Reserved.


#include "CoreMinimal.h"
#include "RequiredProgramMainCPPInclude.h"
#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Logging/LogMacros.h"
#include "Misc/CString.h"
#include "Misc/ScopeExit.h"
#include "Misc/StringBuilder.h"
#include "String/LexFromString.h"

DEFINE_LOG_CATEGORY_STATIC(LogUnrealObjectPtrTool, Log, All);

IMPLEMENT_APPLICATION(UnrealObjectPtrTool, "UnrealObjectPtrTool");

//////////////////////////////////////////////////////////////////////////

enum class EPointerUpgradeBehaviorFlags
{
	None,
	PreviewOnly = 1,
	Reverse     = 2,
};
ENUM_CLASS_FLAGS(EPointerUpgradeBehaviorFlags)

class FPointerUpgrader
{
public:

	FPointerUpgrader(const FString& InSCCCommand, EPointerUpgradeBehaviorFlags InBehaviorFlags) : SCCCommand(InSCCCommand), BehaviorFlags(InBehaviorFlags)
	{
	}

	struct FMemberDeclaration
	{
		int32 LineNumber = 0;
		int32 LineCount = 0;
		FString TypeName;

		bool operator==(const FMemberDeclaration& Other) const
		{
			return (LineNumber == Other.LineNumber) && (TypeName == Other.TypeName);
		}
	};
	using FPointerUpgradeList = TMap<FString, TArray<FMemberDeclaration>>;

	TValueOrError<FPointerUpgradeList, FString> FillUpgradeList(const FString& LogFilename);
	bool TryAllUpgrades(const FPointerUpgradeList& UpgradeList);

	int32 GetTotalUpgradesFound() const { return TotalUpgradesFound; }
	int32 GetTotalUpgradesPerformed() const { return TotalUpgradesPerformed; }

private:
	enum class EByteOrderMarkerType
	{
		None,
		UTF8,
		UCS2LittleEndian,
		UCS2BigEndian,
	};

	struct FPendingWrite
	{
		EByteOrderMarkerType BOM;
		TArray<FString> Contents;
	};
	using FPendingWrites = TMap<FString, FPendingWrite>;
	bool TryUpgradesInFile(const FString& FilenameToUpgrade, TConstArrayView<FMemberDeclaration> MemberDeclarations, FPendingWrites& PendingWrites);
	bool TryFlushPendingWrites(FPendingWrites& PendingWrites);

	// Using these methods instead of the ones in FFileHelper to ensure that we retain the encoding and BOM of the original file as well as the newlines
	static bool LoadFileToString(FString& OutResult, EByteOrderMarkerType& OutBOM, const TCHAR* InFilename, uint32 ReadFlags = 0);
	static bool LoadFileToStringArrayWithNewlines(TArray<FString>& OutResult, EByteOrderMarkerType& OutBOM, const TCHAR* InFilename, uint32 ReadFlags = 0);
	static bool SaveStringToFile(const FStringView& InString, EByteOrderMarkerType InBOM, const TCHAR* InFilename, uint32 WriteFlags = 0);
	static bool SaveStringArrayWithNewlinesToFile(const TArray<FString>& InStringArray, EByteOrderMarkerType InBOM, const TCHAR* InFilename, uint32 WriteFlags = 0);

	static int32 CountLines(const FStringView Input);
	static FString PerformForwardTypenameUpgrade(const FString& TypeName);
	static FString PerformReverseTypenameUpgrade(const FString& TypeName);


	int32 TotalUpgradesFound = 0;
	int32 TotalUpgradesPerformed = 0;
	FString SCCCommand;
	EPointerUpgradeBehaviorFlags BehaviorFlags;
};

FStringView ParseBracketedStringSegment(const FString& InString, int32& InOutCurrentIndex, const TCHAR* BracketStart, const TCHAR* BracketEnd)
{
	int32 ParsedStartIndex = 0;

	if (BracketStart != nullptr)
	{
		ParsedStartIndex = InString.Find(BracketStart, ESearchCase::IgnoreCase, ESearchDir::FromStart, InOutCurrentIndex);
		if (ParsedStartIndex == INDEX_NONE)
		{
			return FStringView();
		}
		ParsedStartIndex += FCString::Strlen(BracketStart);
	}

	int32 ParsedEndIndex = InString.Find(BracketEnd, ESearchCase::IgnoreCase, ESearchDir::FromStart, ParsedStartIndex);
	if (ParsedEndIndex == INDEX_NONE)
	{
		return FStringView();
	}
	InOutCurrentIndex = ParsedEndIndex + FCString::Strlen(BracketEnd);
	return FStringView(*InString + ParsedStartIndex, ParsedEndIndex - ParsedStartIndex);
}

TValueOrError<FPointerUpgrader::FPointerUpgradeList, FString> FPointerUpgrader::FillUpgradeList(const FString& LogFilename)
{
	FPointerUpgradeList UpgradeList;
	const TCHAR* Preamble = EnumHasAllFlags(BehaviorFlags, EPointerUpgradeBehaviorFlags::Reverse) ? TEXT("ObjectPtr usage in member declaration detected [[") : TEXT("Native pointer usage in member declaration detected [[");
	TArray<FString> PointerUpgradeEntries;
	if (!FFileHelper::LoadFileToStringArrayWithPredicate(PointerUpgradeEntries, *LogFilename, [&Preamble](const FString& Line) { return Line.Contains(Preamble);  }))
	{
		return MakeError(FString::Printf(TEXT("Unable to load UHT log: %s"), *LogFilename));
	}

	const int32 PreambleLen = FCString::Strlen(Preamble);
	for (const FString& Entry : PointerUpgradeEntries)
	{
		int32 CurrentIndex = 0;
		FStringView FilenameView = ParseBracketedStringSegment(Entry, CurrentIndex, nullptr, TEXT("("));
		if (FilenameView.IsEmpty())
		{
			return MakeError(FString::Printf(TEXT("Failed to parse filename from pointer member upgrade statement: %s"), *Entry));
		}

		CurrentIndex--; // Backtrack one character to allow us to re-find the opening bracket for the line number segment
		FStringView LineNumberView = ParseBracketedStringSegment(Entry, CurrentIndex, TEXT("("), TEXT(")"));
		if (LineNumberView.IsEmpty())
		{
			return MakeError(FString::Printf(TEXT("Failed to parse line number from pointer member upgrade statement: %s"), *Entry));
		}

		FStringView TypeNameView = ParseBracketedStringSegment(Entry, CurrentIndex, TEXT("[[["), TEXT("]]]"));
		if (TypeNameView.IsEmpty())
		{
			return MakeError(FString::Printf(TEXT("Failed to parse type name from pointer member upgrade statement: %s"), *Entry));
		}

		int32 LineNumber = -1;
		LexFromString(LineNumber, LineNumberView);
		if (LineNumber < 1)
		{
			return MakeError(FString::Printf(TEXT("Failed to convert line number to integer from pointer member upgrade statement: %s"), *Entry));
		}

		if (EnumHasAllFlags(BehaviorFlags, EPointerUpgradeBehaviorFlags::Reverse))
		{
			if (!TypeNameView.StartsWith(TEXT("TObjectPtr<")))
			{
				return MakeError(FString::Printf(TEXT("Failed to find expected 'TObjectPtr<' at end of pointer member upgrade statement: %s"), *Entry));
			}
			if (!TypeNameView.EndsWith(TEXT(">")))
			{
				return MakeError(FString::Printf(TEXT("Failed to find expected angle bracket at end of pointer member upgrade statement: %s"), *Entry));
			}
		}
		else
		{
			if (!TypeNameView.EndsWith(TEXT("*")))
			{
				return MakeError(FString::Printf(TEXT("Failed to find expected asterisk at end of pointer member upgrade statement: %s"), *Entry));
			}
		}

		TArray<FMemberDeclaration>& DeclarationArray = UpgradeList.FindOrAdd(FString(FilenameView));
		FMemberDeclaration NewDecl;
		NewDecl.LineNumber = LineNumber;
		NewDecl.TypeName = TypeNameView;
		NewDecl.TypeName = NewDecl.TypeName.ReplaceEscapedCharWithChar();
		NewDecl.LineCount = CountLines(NewDecl.TypeName);
		int32 DeclArraySize = DeclarationArray.Num();
		DeclarationArray.AddUnique(NewDecl);
		if (DeclArraySize != DeclarationArray.Num())
		{
			TotalUpgradesFound++;
		}
	}

	return MakeValue(MoveTemp(UpgradeList));
}

// The following requirements need to be kept in mind while upgrading source files:
// 1) If the file section we're updating isn't what we expect it to be, we shouldn't attempt the upgrade
// 2) We should handle multi-line type declarations
// 3) We should handle multiple type replacements on the same line
// 4) We should retain the encoding of the original file
// 5) We should retain the newlines of the original file
// 6) We should attempt to retain the indentation (before and after type declaration) of any lines we modify
bool FPointerUpgrader::TryUpgradesInFile(const FString& FilenameToUpgrade, TConstArrayView<FMemberDeclaration> MemberDeclarations, FPendingWrites& PendingWrites)
{
	EByteOrderMarkerType OriginalBOM;
	TArray<FString> FileContentsToUpgrade;
	if (!LoadFileToStringArrayWithNewlines(FileContentsToUpgrade, OriginalBOM, *FilenameToUpgrade))
	{
		UE_LOGF(LogUnrealObjectPtrTool, Error, "Unable to load file to upgrade: %ls", *FilenameToUpgrade);
		return true;
	}

	bool bChangesMadeToFileContents = false;
	for (const FMemberDeclaration& MemberDeclaration : MemberDeclarations)
	{
		if (((MemberDeclaration.LineNumber-MemberDeclaration.LineCount) < 0) || (MemberDeclaration.LineNumber > FileContentsToUpgrade.Num()))
		{
			UE_LOGF(LogUnrealObjectPtrTool, Error, "Member declaration line '%d' is out of range for the lines (%d) in file '%ls' ", MemberDeclaration.LineNumber, FileContentsToUpgrade.Num(), *FilenameToUpgrade);
			continue;
		}

		FString LinesToUpgrade;
		for (int32 LineIndex = MemberDeclaration.LineNumber-MemberDeclaration.LineCount; LineIndex < MemberDeclaration.LineNumber; ++LineIndex)
		{
			LinesToUpgrade.Append(FileContentsToUpgrade[LineIndex]);
		}
		FString OriginalLinesToUpgrade = LinesToUpgrade;

		FString UpgradedTypeName = EnumHasAllFlags(BehaviorFlags, EPointerUpgradeBehaviorFlags::Reverse) ? PerformReverseTypenameUpgrade(MemberDeclaration.TypeName) : PerformForwardTypenameUpgrade(MemberDeclaration.TypeName);
		if (LinesToUpgrade.ReplaceInline(*MemberDeclaration.TypeName, *UpgradedTypeName) < 1)
		{
			if (!LinesToUpgrade.Contains(*UpgradedTypeName))
			{
				UE_LOGF(LogUnrealObjectPtrTool, Warning, "%ls(%d): Type name string '%ls' not found for upgrading, skipping upgrading this line:\n\t%ls", *FilenameToUpgrade, MemberDeclaration.LineNumber, *MemberDeclaration.TypeName, *LinesToUpgrade.TrimStartAndEnd());
			}
			continue;
		}

		TArray<FTextRange> UpgradedTextRanges;
		FTextRange::CalculateLineRangesFromString(LinesToUpgrade, UpgradedTextRanges);

		for (int32 LineIndex = 0; LineIndex < MemberDeclaration.LineCount; ++LineIndex)
		{
			int32 BeginIndex = UpgradedTextRanges[LineIndex].BeginIndex;
			int32 EndIndex = LineIndex < UpgradedTextRanges.Num()-1 ? UpgradedTextRanges[LineIndex+1].BeginIndex : UpgradedTextRanges[LineIndex].EndIndex;
			FString UpgradedLine = LinesToUpgrade.Mid(BeginIndex, EndIndex - BeginIndex);
			FileContentsToUpgrade[MemberDeclaration.LineNumber - MemberDeclaration.LineCount + LineIndex] = UpgradedLine;
		}

		bChangesMadeToFileContents = true;
		TotalUpgradesPerformed++;
		UE_LOGF(LogUnrealObjectPtrTool, Display, "%ls(%d): %ls\n\tFrom:\t%ls\n\tTo:\t%ls", *FilenameToUpgrade, MemberDeclaration.LineNumber, EnumHasAllFlags(BehaviorFlags, EPointerUpgradeBehaviorFlags::PreviewOnly) ? TEXT("Previewing upgrade") : TEXT("Upgrading"), *OriginalLinesToUpgrade.TrimStartAndEnd(), *LinesToUpgrade.TrimStartAndEnd());
	}

	if (bChangesMadeToFileContents)
	{
		FPendingWrite& NewWrite = PendingWrites.Add(FilenameToUpgrade);
		NewWrite.BOM = OriginalBOM;
		NewWrite.Contents.Append(MoveTemp(FileContentsToUpgrade));
	}

	return true;
}

bool FPointerUpgrader::TryFlushPendingWrites(FPendingWrites& PendingWrites)
{
	if (PendingWrites.IsEmpty())
	{
		return true;
	}

	// Perform a batched SCC command operation
	if (!SCCCommand.IsEmpty())
	{
		TStringBuilder<2048> CompositeFilenamesBuilder;
		for (const FPendingWrites::ElementType& PendingWrite : PendingWrites)
		{
			CompositeFilenamesBuilder.AppendChar(TEXT('"'));
			CompositeFilenamesBuilder.Append(PendingWrite.Key);
			CompositeFilenamesBuilder.Append(TEXT("\" "));
		}

		FString FileSCCCommand(SCCCommand);
		FileSCCCommand.ReplaceInline(TEXT("{Filenames}"), *CompositeFilenamesBuilder);
		if (EnumHasAllFlags(BehaviorFlags, EPointerUpgradeBehaviorFlags::PreviewOnly))
		{
			UE_LOGF(LogUnrealObjectPtrTool, Display, "Would perform SCC command: %ls", *FileSCCCommand);
		}
		else
		{
			UE_LOGF(LogUnrealObjectPtrTool, Display, "Perform SCC command: %ls", *FileSCCCommand);
		}

		if (!EnumHasAllFlags(BehaviorFlags, EPointerUpgradeBehaviorFlags::PreviewOnly))
		{
			FString Params = FCommandLine::RemoveExeName(*FileSCCCommand);
			FString Cmd = FileSCCCommand.LeftChop(Params.Len()).TrimEnd();

			FProcHandle ProcHandle = FPlatformProcess::CreateProc(*Cmd, *Params, false, false, false, nullptr, 0, nullptr, nullptr);
			if (!ProcHandle.IsValid())
			{
				UE_LOGF(LogUnrealObjectPtrTool, Error, "Failed to perform SCC command: %ls", *FileSCCCommand);
				return false;
			}

			FPlatformProcess::WaitForProc(ProcHandle);
			int32 RetCode = -1;
			FPlatformProcess::GetProcReturnCode(ProcHandle, &RetCode);
			FPlatformProcess::CloseProc(ProcHandle);

			if (RetCode != 0)
			{
				UE_LOGF(LogUnrealObjectPtrTool, Error, "SCC command returned an unexpected return code: %d", RetCode);
				return false;
			}
		}
	}

	for (const FPendingWrites::ElementType& PendingWrite : PendingWrites)
	{
		if (EnumHasAllFlags(BehaviorFlags, EPointerUpgradeBehaviorFlags::PreviewOnly))
		{
			UE_LOGF(LogUnrealObjectPtrTool, Display, "File would be modified and saved: %ls", *PendingWrite.Key);
		}
		else
		{
			UE_LOGF(LogUnrealObjectPtrTool, Display, "Saving modified file: %ls", *PendingWrite.Key);
			if (!SaveStringArrayWithNewlinesToFile(PendingWrite.Value.Contents, PendingWrite.Value.BOM, *PendingWrite.Key))
			{
				UE_LOGF(LogUnrealObjectPtrTool, Error, "Failed to save upgraded file: %ls", *PendingWrite.Key);
				return false;
			}
		}
	}

	PendingWrites.Empty();
	return true;
}

bool FPointerUpgrader::TryAllUpgrades(const FPointerUpgradeList& UpgradeList)
{
	FPendingWrites PendingWrites;
	for (const TPair<FString, TArray<FMemberDeclaration>>& FileToUpgrade : UpgradeList)
	{
		if (!TryUpgradesInFile(FileToUpgrade.Key, FileToUpgrade.Value, PendingWrites))
		{
			return false;
		}

		if (PendingWrites.Num() >= 20)
		{
			if (!TryFlushPendingWrites(PendingWrites))
			{
				return false;
			}
			check(PendingWrites.IsEmpty());
		}
	}

	if (!TryFlushPendingWrites(PendingWrites))
	{
		return false;
	}
	return true;
}

bool FPointerUpgrader::LoadFileToString(FString& OutResult, EByteOrderMarkerType& OutBOM, const TCHAR* InFilename, uint32 ReadFlags)
{
	TUniquePtr<FArchive> Reader(IFileManager::Get().CreateFileReader(InFilename, ReadFlags));
	if (!Reader)
	{
		return false;
	}

	FScopedLoadingState ScopedLoadingState(*Reader->GetArchiveName());

	int64 Size = Reader->TotalSize();
	if (!Size)
	{
		OutBOM = EByteOrderMarkerType::None;
		OutResult.Empty();
		return true;
	}

	uint8* OriginalBuffer = (uint8*)FMemory::Malloc(Size);
	uint8* Buffer = OriginalBuffer;
	ON_SCOPE_EXIT
	{
		FMemory::Free(OriginalBuffer);
	};

	Reader->Serialize(Buffer, Size);
	if (!Reader->Close())
	{
		return false;
	}

	TArray<TCHAR, FString::AllocatorType>& ResultArray = OutResult.GetCharArray();
	ResultArray.Empty();

	bool bIsUnicode = false;
	if (Size >= 2 && !(Size & 1) && Buffer[0] == 0xff && Buffer[1] == 0xfe)
	{
		// Unicode Intel byte order. Less 1 for the FFFE header, additional 1 for null terminator.
		OutBOM = EByteOrderMarkerType::UCS2LittleEndian;
		ResultArray.AddUninitialized(Size / 2);
		for (int32 i = 0; i < (Size / 2) - 1; i++)
		{
			ResultArray[i] = CharCast<TCHAR>((UCS2CHAR)((uint16)Buffer[i * 2 + 2] + (uint16)Buffer[i * 2 + 3] * 256));
		}
		bIsUnicode = true;
	}
	else if (Size >= 2 && !(Size & 1) && Buffer[0] == 0xfe && Buffer[1] == 0xff)
	{
		// Unicode non-Intel byte order. Less 1 for the FFFE header, additional 1 for null terminator.
		OutBOM = EByteOrderMarkerType::UCS2BigEndian;
		ResultArray.AddUninitialized(Size / 2);
		for (int32 i = 0; i < (Size / 2) - 1; i++)
		{
			ResultArray[i] = CharCast<TCHAR>((UCS2CHAR)((uint16)Buffer[i * 2 + 3] + (uint16)Buffer[i * 2 + 2] * 256));
		}
		bIsUnicode = true;
	}
	else
	{
		if (Size >= 3 && Buffer[0] == 0xef && Buffer[1] == 0xbb && Buffer[2] == 0xbf)
		{
			// Skip over UTF-8 BOM if there is one
			OutBOM = EByteOrderMarkerType::UTF8;
			Buffer += 3;
			Size -= 3;
		}
		else
		{
			OutBOM = EByteOrderMarkerType::None;
		}

		int32 Length = FUTF8ToTCHAR_Convert::ConvertedLength(reinterpret_cast<const ANSICHAR*>(Buffer), Size);
		ResultArray.AddUninitialized(Length + 1); // +1 for the null terminator
		FUTF8ToTCHAR_Convert::Convert(ResultArray.GetData(), ResultArray.Num(), reinterpret_cast<const ANSICHAR*>(Buffer), Size);
		ResultArray[Length] = TEXT('\0');
	}

	if (ResultArray.Num() == 1)
	{
		// If it's only a zero terminator then make the result actually empty
		ResultArray.Empty();
	}
	else
	{
		// Else ensure null terminator is present
		ResultArray.Last() = 0;
	}

	return true;
}

bool FPointerUpgrader::LoadFileToStringArrayWithNewlines(TArray<FString>& OutResult, EByteOrderMarkerType& OutBOM, const TCHAR* InFilename, uint32 ReadFlags)
{
	FString FileContents;
	if (!LoadFileToString(FileContents, OutBOM, InFilename, ReadFlags))
	{
		return false;
	}

	TArray<FTextRange> LineRanges;
	FTextRange::CalculateLineRangesFromString(FileContents, LineRanges);
	OutResult.Reset(LineRanges.Num());

	for (int32 LineIndex = 0; LineIndex < LineRanges.Num(); ++LineIndex)
	{
		int32 BeginIndex = LineRanges[LineIndex].BeginIndex;
		int32 EndIndex = (LineIndex < (LineRanges.Num() - 1)) ? LineRanges[LineIndex+1].BeginIndex : LineRanges[LineIndex].EndIndex;
		OutResult.Add(FileContents.Mid(BeginIndex, EndIndex - BeginIndex));
	}

	return true;
}

bool FPointerUpgrader::SaveStringToFile(const FStringView& InString, EByteOrderMarkerType InBOM, const TCHAR* InFilename, uint32 WriteFlags)
{
	// max size of the string is a UCS2CHAR for each character and some UNICODE magic 
	TUniquePtr<FArchive> Ar = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(InFilename, WriteFlags));
	if (!Ar)
	{
		return false;
	}

	if (InString.IsEmpty())
	{
		return true;
	}

	if (InBOM == EByteOrderMarkerType::UTF8)
	{
		UTF8CHAR UTF8BOM[] = {(UTF8CHAR)0xEF, (UTF8CHAR)0xBB, (UTF8CHAR)0xBF};
		Ar->Serialize(&UTF8BOM, UE_ARRAY_COUNT(UTF8BOM) * sizeof(UTF8CHAR));

		FTCHARToUTF8 UTF8String(InString.GetData(), InString.Len());
		Ar->Serialize((UTF8CHAR*)UTF8String.Get(), UTF8String.Length() * sizeof(UTF8CHAR));
	}
	else if ((InBOM == EByteOrderMarkerType::UCS2LittleEndian) || (InBOM == EByteOrderMarkerType::UCS2BigEndian))
	{
		Ar->SetByteSwapping(PLATFORM_LITTLE_ENDIAN ? InBOM == EByteOrderMarkerType::UCS2BigEndian : InBOM == EByteOrderMarkerType::UCS2LittleEndian);
		
		UTF16CHAR BOM = UNICODE_BOM;
		Ar->Serialize(&BOM, sizeof(UTF16CHAR));

		// Note: This is a no-op on platforms that are using a 16-bit TCHAR
		FTCHARToUTF16 UTF16String(InString.GetData(), InString.Len());
		Ar->Serialize((UTF16CHAR*)UTF16String.Get(), UTF16String.Length() * sizeof(UTF16CHAR));
	}
	else
	{
		// If there is no BOM specified, write to UTF8 without a BOM.  If the contents fit within ASCII, they will be
		// written as ASCII.  If they don't fit, they'll be written as multibyte code points.  This is necessary as
		// a file was encountered that is UTF8 without any BOM.  The reading code handles this correctly,
		// so we want to ensure that the writing code doesn't substitute "bogus chars" (?) for any non-ASCII characters
		// at the time of writing.
		FTCHARToUTF8 UTF8String(InString.GetData(), InString.Len());
		Ar->Serialize((UTF8CHAR*)UTF8String.Get(), UTF8String.Length() * sizeof(UTF8CHAR));
	}

	// Always explicitly close to catch errors from flush/close
	Ar->Close();

	return !Ar->IsError() && !Ar->IsCriticalError();
}

bool FPointerUpgrader::SaveStringArrayWithNewlinesToFile(const TArray<FString>& InStringArray, EByteOrderMarkerType InBOM, const TCHAR* InFilename, uint32 WriteFlags)
{
	FString ComposedFileContents = FString::Join(InStringArray, TEXT(""));
	return SaveStringToFile(ComposedFileContents, InBOM, InFilename, WriteFlags);
}

int32 FPointerUpgrader::CountLines(const FStringView Input)
{
	int32 LineCount = 0;
	int32 LineBeginIndex = 0;

	// Loop through splitting at new-lines
	const TCHAR* const InputStart = Input.GetData();
	const TCHAR* const InputEnd = InputStart + Input.Len();
	for (const TCHAR* CurrentChar = InputStart; CurrentChar != InputEnd; ++CurrentChar)
	{
		// Handle a chain of \r\n slightly differently to stop the FChar::IsLinebreak adding two separate new-lines
		const bool bIsWindowsNewLine = ((CurrentChar + 1) != InputEnd) && (*CurrentChar == '\r' && *(CurrentChar + 1) == '\n');
		if (bIsWindowsNewLine || FChar::IsLinebreak(*CurrentChar))
		{
			++LineCount;

			if (bIsWindowsNewLine)
			{
				++CurrentChar; // skip the \n of the \r\n chain
			}
			LineBeginIndex = UE_PTRDIFF_TO_INT32(CurrentChar - InputStart) + 1; // The next line begins after the end of the current line
		}
	}

	// Process any remaining string after the last new-line
	if (LineBeginIndex <= Input.Len())
	{
		++LineCount;
	}

	return LineCount;
}

FString FPointerUpgrader::PerformForwardTypenameUpgrade(const FString& TypeName)
{
	// Chop off the trailing asterisk in the type declaration
	FString TemplateArg = TypeName.LeftChop(1);
	// Any trailing whitespace that was after the asterisk should not go in the template argument, but be appended after the template argument
	// This only applies to space (' ') and tab ('\t') that have non-whitespace before them on the line, and should not include newlines.
	// This is to avoid having end template bracket ('>') get put on a line that may have a line comment ahead of it ("//").
	int32 TrailingWhitespaceCount = 0;
	int32 CharIndex;
	for (CharIndex = TemplateArg.Len()-1; (CharIndex >= 0) && (TemplateArg[CharIndex] == TCHAR(' ') || TemplateArg[CharIndex] == TCHAR('\t')); --CharIndex)
	{
		++TrailingWhitespaceCount;
	}
	if ((CharIndex == 0) || (TemplateArg[CharIndex-1] == TEXT('\n')) || (TemplateArg[CharIndex-1] == TEXT('\r')))
	{
		// The trailing whitespace represented the entire lead of the line the end template character will be on.  Put the indentation inside the template arg instead of outside of it.
		TrailingWhitespaceCount = 0;
	}
	FString TrailingWhitespace = TemplateArg.Right(TrailingWhitespaceCount);
	TemplateArg.LeftChopInline(TrailingWhitespaceCount);
	return FString::Printf(TEXT("TObjectPtr<%s>%s"), *TemplateArg, *TrailingWhitespace);
}

FString FPointerUpgrader::PerformReverseTypenameUpgrade(const FString& TypeName)
{
	// Chop off the trailing angle bracket in the type declaration
	FString TemplateArg = TypeName.LeftChop(1);
	// Chop off the leading TObjectPtr in the type declaration
	TemplateArg = TemplateArg.RightChop(UE_ARRAY_COUNT(TEXT("TObjectPtr<")) - 1);

	return FString::Printf(TEXT("%s*"), *TemplateArg);
}

//////////////////////////////////////////////////////////////////////////

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	FTaskTagScope Scope(ETaskTag::EGameThread);
	FString CmdLine = FCommandLine::BuildFromArgV(nullptr, ArgC, ArgV, nullptr);
	GEngineLoop.PreInit(*CmdLine);

	// Make sure the engine is properly cleaned up whenever we exit this function
	ON_SCOPE_EXIT
	{
		FEngineLoop::AppPreExit();
		FEngineLoop::AppExit();
	};

	FString UHTLogFilename;
	if (CmdLine.Len() && **CmdLine != TEXT('-'))
	{
		const TCHAR* CmdLinePtr = *CmdLine;
		UHTLogFilename = FParse::Token(CmdLinePtr, false);
	}

	if (UHTLogFilename.IsEmpty())
	{
		UHTLogFilename = FPaths::ConvertRelativePathToFull(FPlatformProcess::BaseDir(), TEXT("../../../Engine/Programs/UnrealHeaderTool/Saved/Logs/UnrealHeaderTool.log"));
	}
	
	EPointerUpgradeBehaviorFlags BehaviorFlags = EPointerUpgradeBehaviorFlags::None;
	FString SCCCommand;
	FParse::Value(*CmdLine, TEXT("-SCCCommand="), SCCCommand);
	if (FParse::Param(*CmdLine, TEXT("n")) || FParse::Param(*CmdLine, TEXT("PREVIEW")))
	{
		BehaviorFlags |= EPointerUpgradeBehaviorFlags::PreviewOnly;
	}
	if (FParse::Param(*CmdLine, TEXT("r")) || FParse::Param(*CmdLine, TEXT("REVERSE")))
	{
		BehaviorFlags |= EPointerUpgradeBehaviorFlags::Reverse;
	}

	FPointerUpgrader PointerUpgrader(SCCCommand, BehaviorFlags);

	TValueOrError<FPointerUpgrader::FPointerUpgradeList, FString> UpgradeListResult =
		PointerUpgrader.FillUpgradeList(UHTLogFilename);

	if (!UpgradeListResult.IsValid())
	{
		UE_LOGF(LogUnrealObjectPtrTool, Error, "Upgrade log parsing error: %ls", *UpgradeListResult.GetError());
		return 1;
	}

	if (!PointerUpgrader.TryAllUpgrades(UpgradeListResult.GetValue()))
	{
		return 1;
	}

	if (EnumHasAllFlags(BehaviorFlags, EPointerUpgradeBehaviorFlags::PreviewOnly))
	{
		UE_LOGF(LogUnrealObjectPtrTool, Display, "Previewed upgrade successfully.  %d upgrades found; %d upgrades would be performed", PointerUpgrader.GetTotalUpgradesFound(), PointerUpgrader.GetTotalUpgradesPerformed());
	}
	else
	{
		UE_LOGF(LogUnrealObjectPtrTool, Display, "Upgraded successfully.  %d upgrades found; %d upgrades performed", PointerUpgrader.GetTotalUpgradesFound(), PointerUpgrader.GetTotalUpgradesPerformed());
	}

	return 0;
}
