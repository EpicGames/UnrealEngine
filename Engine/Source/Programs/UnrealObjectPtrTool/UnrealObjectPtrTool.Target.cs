// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class UnrealObjectPtrToolTarget : TargetRules
{
	public UnrealObjectPtrToolTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Modular;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		LaunchModuleName = "UnrealObjectPtrTool";

		bBuildDeveloperTools			= false;
		bBuildWithEditorOnlyData		= false;
		bCompileAgainstEngine			= false;
		bCompileAgainstCoreUObject		= false;
		bCompileAgainstApplicationCore	= false;
		bUsesSlate = false;
		bIsBuildingConsoleApplication	= true;
	}
}
